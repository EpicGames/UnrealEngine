// Copyright Epic Games, Inc. All Rights Reserved.


#include "Decoders/VorbisAudioInfo.h"
#include "Interfaces/IAudioFormat.h"
#include "Audio.h"
#include <ogg/os_types.h>
#include <vorbis/codec.h>
#include "Misc/Paths.h"
#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif

#if WITH_OGGVORBIS

#pragma pack(push, 8)
#include "vorbis/vorbisfile.h"
#pragma pack(pop)
#endif

#if PLATFORM_LITTLE_ENDIAN
#define VORBIS_BYTE_ORDER 0
#else
#define VORBIS_BYTE_ORDER 1
#endif

// Non-windows platform don't load Dlls
#if !WITH_OGGVORBIS_DLL
static FThreadSafeBool bDllLoaded = true;
#else
static FThreadSafeBool bDllLoaded;
#endif

static int32 VorbisReadFailiureTimeoutCVar = 1;
FAutoConsoleVariableRef CVarVorbisReadFailiureTimeout(
	TEXT("au.vorbis.ReadFailiureTimeout"),
	VorbisReadFailiureTimeoutCVar,
	TEXT("When set to 1, we bail on decoding Ogg Vorbis sounds if we were not able to successfully decode them after several attempts.\n"),
	ECVF_Default);

/**
 * Channel order expected for a multi-channel ogg vorbis file.
 * Ordering taken from http://xiph.org/vorbis/doc/Vorbis_I_spec.html#x1-800004.3.9
 */
namespace VorbisChannelInfo
{
	const int32 Order[8][8] = {
		{ 0 },
		{ 0, 1 },
		{ 0, 2, 1 },
		{ 0, 1, 2, 3 },
		{ 0, 2, 1, 3, 4 },
		{ 0, 2, 1, 4, 5, 3 },
		{ 0, 2, 1, 4, 5, 6, 3 },
		{ 0, 2, 1, 4, 5, 6, 7, 3 }
	};
};

/*------------------------------------------------------------------------------------
	FVorbisFileWrapper. Hides Vorbis structs from public headers.
------------------------------------------------------------------------------------*/
struct FVorbisFileWrapper
{
	FVorbisFileWrapper()
	{
#if WITH_OGGVORBIS
		FMemory::Memzero( &vf, sizeof( OggVorbis_File ) );
#endif
	}

	~FVorbisFileWrapper()
	{
#if WITH_OGGVORBIS
		// Only clear vorbis if the DLL succeeded in loading
		if (bDllLoaded)
		{
			ov_clear(&vf);
		}
#endif
	}

#if WITH_OGGVORBIS
	/** Ogg vorbis decompression state */
	OggVorbis_File	vf;
#endif
};

#if WITH_OGGVORBIS
/*------------------------------------------------------------------------------------
	FVorbisAudioInfo.
------------------------------------------------------------------------------------*/
FVorbisAudioInfo::FVorbisAudioInfo()
	: VFWrapper(new FVorbisFileWrapper())
	, SrcBufferData(NULL)
	, SrcBufferDataSize(0)
	, BufferOffset(0)
	, CurrentBufferChunkOffset(0)
	, TimesLoopedWithoutDecompressedAudio(0)
	, CurrentStreamingChunkData(nullptr)
	, CurrentStreamingChunkIndex(INDEX_NONE)
	, NextStreamingChunkIndex(0)
	, CurrentStreamingChunksSize(0)
	, bHeaderParsed(false)
{
	// Make sure we have properly allocated a VFWrapper
	check(VFWrapper != NULL);
}

FVorbisAudioInfo::~FVorbisAudioInfo()
{
	FScopeLock ScopeLock(&VorbisCriticalSection);
	check(VFWrapper != nullptr);
	delete VFWrapper;
	VFWrapper = nullptr;
}

/** Emulate read from memory functionality */
size_t FVorbisAudioInfo::ReadMemory( void *Ptr, uint32 Size )
{
	check(Ptr);
	size_t BytesToRead = FMath::Min(Size, SrcBufferDataSize - BufferOffset);
	FMemory::Memcpy( Ptr, SrcBufferData + BufferOffset, BytesToRead );
	BufferOffset += BytesToRead;
	return( BytesToRead );
}

static size_t OggReadMemory( void *ptr, size_t size, size_t nmemb, void *datasource )
{
	check(ptr);
	check(datasource);
	FVorbisAudioInfo* OggInfo = (FVorbisAudioInfo*)datasource;
	return( OggInfo->ReadMemory( ptr, size * nmemb ) );
}

int FVorbisAudioInfo::SeekMemory( uint32 offset, int whence )
{
	switch( whence )
	{
	case SEEK_SET:
		BufferOffset = offset;
		break;

	case SEEK_CUR:
		BufferOffset += offset;
		break;

	case SEEK_END:
		BufferOffset = SrcBufferDataSize - offset;
		break;

	default:
		checkf(false, TEXT("Uknown seek type"));
		break;
	}

	BufferOffset = FMath::Clamp<uint32>(BufferOffset, 0, SrcBufferDataSize);
	return( BufferOffset );
}

static int OggSeekMemory( void *datasource, ogg_int64_t offset, int whence )
{
	FVorbisAudioInfo* OggInfo = ( FVorbisAudioInfo* )datasource;
	return( OggInfo->SeekMemory( offset, whence ) );
}

int FVorbisAudioInfo::CloseMemory( void )
{
	return( 0 );
}

static int OggCloseMemory( void *datasource )
{
	FVorbisAudioInfo* OggInfo = ( FVorbisAudioInfo* )datasource;
	return( OggInfo->CloseMemory() );
}

long FVorbisAudioInfo::TellMemory( void )
{
	return( BufferOffset );
}

static long OggTellMemory( void *datasource )
{
	FVorbisAudioInfo *OggInfo = ( FVorbisAudioInfo* )datasource;
	return( OggInfo->TellMemory() );
}

/** Emulate read from memory functionality */
size_t FVorbisAudioInfo::ReadStreaming(void *Ptr, uint32 Size )
{
	size_t NumBytesRead = 0;

	while (NumBytesRead < Size)
	{
		if (!CurrentStreamingChunkData || CurrentStreamingChunkIndex != NextStreamingChunkIndex)
		{
			CurrentStreamingChunkIndex = NextStreamingChunkIndex;

			check(CurrentStreamingChunkIndex >= 0);

			if (static_cast<uint32>(CurrentStreamingChunkIndex) >= StreamingSoundWaveData->GetNumChunks())
			{
				CurrentStreamingChunkData = nullptr;
				CurrentStreamingChunksSize = 0;
			}
			else
			{
				CurrentStreamingChunkData = GetLoadedChunk(StreamingSoundWaveData.ToSharedRef(), CurrentStreamingChunkIndex, CurrentStreamingChunksSize);
			}

			if (CurrentStreamingChunkData)
			{
				check(CurrentStreamingChunkIndex < (int32)StreamingSoundWaveData->GetNumChunks());
				CurrentBufferChunkOffset = 0;
			}
		}

		// No chunk data -- either looping or something else happened with stream
		if (!CurrentStreamingChunkData)
		{
			return NumBytesRead;
		}

		check(CurrentStreamingChunksSize >= CurrentBufferChunkOffset);
		// How many bytes left in the current chunk
		uint32 BytesLeftInCurrentChunk = CurrentStreamingChunksSize - CurrentBufferChunkOffset;

		check(Size >= NumBytesRead);
		// How many more bytes we want to read
		uint32 NumBytesLeftToRead = Size - NumBytesRead;

		// The amount of audio we're going to copy is the min of the bytes left in the chunk and the bytes left we need to read.
		size_t BytesToCopy = FMath::Min(BytesLeftInCurrentChunk, NumBytesLeftToRead);
		if (BytesToCopy > 0)
		{
			void* WriteBufferLocation = (void*)((uint8*)Ptr + NumBytesRead);
			FMemory::Memcpy(WriteBufferLocation, CurrentStreamingChunkData + CurrentBufferChunkOffset, BytesToCopy);

			// Increment the BufferOffset by how many bytes we copied from the stream
			BufferOffset += BytesToCopy;

			// Increment the current buffer's offset
			CurrentBufferChunkOffset += BytesToCopy;

			// Increment the number of bytes we read this callback.
			NumBytesRead += BytesToCopy;
		}

		// If we need to read more bytes than are left in the current chunk, we're going to need to increment the chunk index so we read the next chunk of audio
		if (NumBytesLeftToRead >= BytesLeftInCurrentChunk)
		{
			NextStreamingChunkIndex++;
		}
	}

	return NumBytesRead;
}

static size_t OggReadStreaming( void *ptr, size_t size, size_t nmemb, void *datasource )
{
	check(ptr);
	check(datasource);
	if (FVorbisAudioInfo* OggInfo = (FVorbisAudioInfo*)datasource)
	{
		return OggInfo->ReadStreaming(ptr, size * nmemb);
	}
	UE_LOGF(LogAudio, Error, "OggReadStreaming had null audio info datasource.");
	return 0;
}

int FVorbisAudioInfo::CloseStreaming( void )
{
	return( 0 );
}

static int OggCloseStreaming( void *datasource )
{
	FVorbisAudioInfo* OggInfo = ( FVorbisAudioInfo* )datasource;
	return( OggInfo->CloseStreaming() );
}

#define CASE_AND_STRING(X) case X: return TEXT(#X)

static const TCHAR* GetVorbisError(const int32 InErrorCode)
{
	switch (InErrorCode)
	{
		CASE_AND_STRING(OV_FALSE);
		CASE_AND_STRING(OV_EOF);
		CASE_AND_STRING(OV_HOLE);

		CASE_AND_STRING(OV_EREAD);
		CASE_AND_STRING(OV_EFAULT);
		CASE_AND_STRING(OV_EIMPL);
		CASE_AND_STRING(OV_EINVAL);
		CASE_AND_STRING(OV_ENOTVORBIS);
		CASE_AND_STRING(OV_EBADHEADER);
		CASE_AND_STRING(OV_EVERSION);
		CASE_AND_STRING(OV_ENOTAUDIO);
		CASE_AND_STRING(OV_EBADPACKET);
		CASE_AND_STRING(OV_EBADLINK);
		CASE_AND_STRING(OV_ENOSEEK);
	default:
		break;
	}
	return TEXT("Unknown");
}

#undef CASE_AND_STRING

bool FVorbisAudioInfo::GetCompressedInfoCommon(void* Callbacks, FSoundQualityInfo* QualityInfo)
{
	if (!bDllLoaded)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::GetCompressedInfoCommon failed due to vorbis DLL not being loaded.");
		bHasError = true;
		return false;
	}

	// Set up the read from memory variables
	int Result = ov_open_callbacks(this, &VFWrapper->vf, NULL, 0, (*(ov_callbacks*)Callbacks));
	if (Result < 0)
	{
		UE_LOGF(LogAudio, Warning, "FVorbisAudioInfo::ReadCompressedInfo, ov_open_callbacks error code: %d : %ls", Result,GetVorbisError(Result));
		bHasError = true;
		LastErrorCode = Result;
		return false;
	}

	if( QualityInfo )
	{
		// The compression could have resampled the source to make loopable
		vorbis_info* vi = ov_info( &VFWrapper->vf, -1 );
		QualityInfo->SampleRate = vi->rate;
		QualityInfo->NumChannels = vi->channels;
		ogg_int64_t PCMTotal = ov_pcm_total( &VFWrapper->vf, -1 );
		if (PCMTotal >= 0)
		{
			QualityInfo->SampleDataSize = PCMTotal * QualityInfo->NumChannels * sizeof( int16 );
			QualityInfo->Duration = ( float )ov_time_total( &VFWrapper->vf, -1 );
		}
		else if (PCMTotal == OV_EINVAL)
		{
			// indicates an error or that the bitstream is non-seekable
			QualityInfo->SampleDataSize = 0;
			QualityInfo->Duration = 0.0f;
		}
	}

	return true;
}

/**
 * Reads the header information of an ogg vorbis file
 *
 * @param	Resource		Info about vorbis data
 */
bool FVorbisAudioInfo::ReadCompressedInfo( const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, FSoundQualityInfo* QualityInfo )
{
	if (!bDllLoaded)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::ReadCompressedInfo failed due to vorbis DLL not being loaded.");
		return false;
	}
	
	if (bHeaderParsed)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::ReadCompressedInfo failed due to the header being parsed already.");
		return false;
	}

	if (!VFWrapper)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::ReadCompressedInfo failed due to not having vorbis wrapper.");
		return false;
	}
	

	FScopeLock ScopeLock(&VorbisCriticalSection);

	ov_callbacks Callbacks;

	SrcBufferData = InSrcBufferData;
	SrcBufferDataSize = InSrcBufferDataSize;
	BufferOffset = 0;

	Callbacks.read_func = OggReadMemory;
	Callbacks.seek_func = OggSeekMemory;
	Callbacks.close_func = OggCloseMemory;
	Callbacks.tell_func = OggTellMemory;

	bHeaderParsed = GetCompressedInfoCommon(&Callbacks, QualityInfo);

	if (!bHeaderParsed)
	{	
		UE_LOGF(LogAudio, Error, "Failed to parse header for compressed vorbis file.");
	}


	return bHeaderParsed;
}


/**
 * Decompress an entire ogg vorbis data file to a TArray
 */
void FVorbisAudioInfo::ExpandFile( uint8* DstBuffer, FSoundQualityInfo* QualityInfo )
{
	if (!bDllLoaded)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::ExpandFile failed due to vorbis DLL not being loaded.");
		return;
	}

	uint32		TotalBytesRead, BytesToRead;

	check( VFWrapper != NULL );
	check( DstBuffer );
	check( QualityInfo );

	FScopeLock ScopeLock(&VorbisCriticalSection);

	if (!bHeaderParsed)
	{
		UE_LOGF(LogAudio, Error, "Failed to expand vorbis file to not parsing header first.");
		return;
	}

	// A zero buffer size means decompress the entire ogg vorbis stream to PCM.
	TotalBytesRead = 0;
	BytesToRead = QualityInfo->SampleDataSize;

	char* Destination = ( char* )DstBuffer;
	while( TotalBytesRead < BytesToRead )
	{
		long BytesRead = ov_read( &VFWrapper->vf, Destination, BytesToRead - TotalBytesRead, 0, 2, 1, NULL );

		if (BytesRead < 0)
		{
			// indicates an error - fill remainder of buffer with zero
			FMemory::Memzero(Destination, BytesToRead - TotalBytesRead);
			return;
		}

		TotalBytesRead += BytesRead;
		Destination += BytesRead;
	}
}



/**
 * Decompresses ogg vorbis data to raw PCM data.
 *
 * @param	PCMData		where to place the decompressed sound
 * @param	bLooping	whether to loop the wav by seeking to the start, or pad the buffer with zeroes
 * @param	BufferSize	number of bytes of PCM data to create. A value of 0 means decompress the entire sound.
 *
 * @return	bool		true if the end of the data was reached (for both single shot and looping sounds)
 */

bool FVorbisAudioInfo::ReadCompressedData( uint8* InDestination, bool bLooping, uint32 BufferSize )
{
	if (!bDllLoaded)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::ReadCompressedData failed due to vorbis DLL not being loaded.");
		return true;
	}

	bool		bLooped;
	uint32		TotalBytesRead;



	FScopeLock ScopeLock(&VorbisCriticalSection);

	if (!bHeaderParsed)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::ReadCompressedData failed due to not parsing header first.");
		return true;
	}

	bLooped = false;

	// Work out number of samples to read
	TotalBytesRead = 0;
	char* Destination = ( char* )InDestination;

	check( VFWrapper != NULL );

	while( TotalBytesRead < BufferSize )
	{
		long BytesRead = ov_read( &VFWrapper->vf, Destination, BufferSize - TotalBytesRead, 0, 2, 1, NULL );
		if( !BytesRead )
		{
			// We've reached the end
			bLooped = true;
			if( bLooping )
			{
				int Result = ov_pcm_seek_page( &VFWrapper->vf, 0 );
				if (Result < 0)
				{
					// indicates an error - fill remainder of buffer with zero
					FMemory::Memzero(Destination, BufferSize - TotalBytesRead);
					return true;
				}
			}
			else
			{
				int32 Count = ( BufferSize - TotalBytesRead );
				FMemory::Memzero( Destination, Count );

				BytesRead += BufferSize - TotalBytesRead;
			}
		}
		else if (BytesRead < 0)
		{
			// indicates an error - fill remainder of buffer with zero
			FMemory::Memzero(Destination, BufferSize - TotalBytesRead);
			return false;
		}

		TotalBytesRead += BytesRead;
		Destination += BytesRead;
	}

	return( bLooped );
}

void FVorbisAudioInfo::SeekToTime( const float SeekTime )
{
	if (!bDllLoaded)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::SeekToTime failed due to vorbis DLL not being loaded.");
		return;
	}

	FScopeLock ScopeLock(&VorbisCriticalSection);

	if (!bHeaderParsed)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::SeekToTime failed due to not parsing header first.");
		return;
	}
	
	if (!VFWrapper->vf.seekable)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::SeekToTime failed due to vorbis file not being seekable");
		return;
	}

	const float TargetTime = FMath::Min(SeekTime, (float)ov_time_total(&VFWrapper->vf, -1));
	ov_time_seek( &VFWrapper->vf, TargetTime );
}

void FVorbisAudioInfo::SeekToFrame(const uint32 SeekFrames)
{
	if (!bDllLoaded)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::SeekToTime failed due to vorbis DLL not being loaded.");
		return;
	}

	FScopeLock ScopeLock(&VorbisCriticalSection);

	if (!bHeaderParsed)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::SeekToTime failed due to not parsing header first.");
		return;
	}

	if (!VFWrapper->vf.seekable)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::SeekToTime failed due to vorbis file not being seekable");
		return;
	}

	vorbis_info* vi = ov_info(&VFWrapper->vf, -1);
	int64 SeekSamples = SeekFrames * vi->channels;
	const int64 TargetSample = FMath::Min(SeekSamples, (int64)ov_pcm_total(&VFWrapper->vf, -1));
	ov_pcm_seek(&VFWrapper->vf, TargetSample);
}

void FVorbisAudioInfo::EnableHalfRate( bool HalfRate )
{
	if (!bDllLoaded)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::EnableHalfRate failed due to vorbis DLL not being loaded.");
		return;
	}

	FScopeLock ScopeLock(&VorbisCriticalSection);

	if (!bHeaderParsed)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::EnableHalfRate failed due to not parsing header first.");
		return;
	}

	ov_halfrate(&VFWrapper->vf, int32(HalfRate));
}

bool FVorbisAudioInfo::StreamCompressedInfoInternal(const TSharedRef<const FSoundWaveData>& InWaveData, struct FSoundQualityInfo* QualityInfo)
{
	if (!bDllLoaded)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::StreamCompressedInfoInternal failed to parse header due to vorbis DLL not being loaded for sound '%ls'.", *InWaveData->GetFName().ToString());
		return false;
	}

	FScopeLock ScopeLock(&VorbisCriticalSection);

	if (!VFWrapper)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::StreamCompressedInfoInternal failed due to no vorbis wrapper for sound '%ls'.", *InWaveData->GetFName().ToString());
		return false;
	}

	ov_callbacks Callbacks;

	SrcBufferData = NULL;
	SrcBufferDataSize = 0;
	BufferOffset = 0;

	Callbacks.read_func = OggReadStreaming;
	Callbacks.close_func = OggCloseStreaming;
	Callbacks.seek_func = NULL;	// Force streaming
	Callbacks.tell_func = NULL;	// Force streaming

	bHeaderParsed = GetCompressedInfoCommon(&Callbacks, QualityInfo);
	if (!bHeaderParsed)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::StreamCompressedInfoInternal failed to parse header for '%ls'. LastError=%ls",
			*InWaveData->GetFName().ToString(), GetVorbisError(LastErrorCode));
	}

	return bHeaderParsed;
}

int32 FVorbisAudioInfo::GetAudioDataStartOffset() const
{
	FScopeLock ScopeLock(&VorbisCriticalSection);

	if (VFWrapper && VFWrapper->vf.dataoffsets)
	{
		return VFWrapper->vf.dataoffsets[0];
	}
	return -1;
}


const uint8* FVorbisAudioInfo::GetLoadedChunk(const TSharedRef<const FSoundWaveData>& InSoundWaveData, uint32 ChunkIndex, uint32& OutChunkSize)
{
	if (ChunkIndex >= InSoundWaveData->GetNumChunks())
	{
		OutChunkSize = 0;
		return nullptr;
	}
	else if (ChunkIndex == 0)
	{
		TArrayView<const uint8> ZerothChunk = InSoundWaveData->GetZerothChunk(true);
		OutChunkSize = ZerothChunk.Num();
		return ZerothChunk.GetData();
	}
	else
	{
		CurCompressedChunkHandle = IStreamingManager::Get().GetAudioStreamingManager().GetLoadedChunk(InSoundWaveData, ChunkIndex, /* bBlockForLoad */ true, true);
		OutChunkSize = CurCompressedChunkHandle.Num();
		return CurCompressedChunkHandle.GetData();
	}
}

bool FVorbisAudioInfo::StreamCompressedData(uint8* InDestination, bool bLooping, uint32 BufferSize, int32& OutNumBytesStreamed)
{
	const uint32 DestinationSize = BufferSize;
	OutNumBytesStreamed = 0;

	if (!bDllLoaded)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::StreamCompressedData failed due to vorbis DLL not being loaded.");
		return true;
	}

	check( VFWrapper != NULL );


	FScopeLock ScopeLock(&VorbisCriticalSection);

	if (!bHeaderParsed)
	{
		UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::StreamCompressedData failed due to not parsing header first.");
		return true;
	}

	bool bLooped = false;

	while( BufferSize > 0 )
	{
		long BytesActuallyRead = ov_read( &VFWrapper->vf, (char*)InDestination, (int)BufferSize, 0, 2, 1, NULL );

		if( BytesActuallyRead <= 0 )
		{
			// if we read 0 bytes or there was an error, instead of assuming we looped, lets write out zero's.
			// this means that the chunk wasn't loaded in time.

			check(StreamingSoundWaveData->GetNumChunks() < ((uint32)TNumericLimits<int32>::Max()));

			if (NextStreamingChunkIndex < static_cast<int32>(StreamingSoundWaveData->GetNumChunks()))
			{
				// zero out the rest of the buffer
				FMemory::Memzero(InDestination, BufferSize);
				return false;
			}

			// We've reached the end
			bLooped = true;
			TimesLoopedWithoutDecompressedAudio++;

			// Clean up decoder state:
			BufferOffset = 0;
			ov_clear(&VFWrapper->vf);
			FMemory::Memzero(&VFWrapper->vf, sizeof(OggVorbis_File));

			constexpr int32 NumReadFailiuresUntilTimeout = 64;
			const bool bDidTimeoutOnDecompressionFailiures = (VorbisReadFailiureTimeoutCVar > 0) && (TimesLoopedWithoutDecompressedAudio >= NumReadFailiuresUntilTimeout);

			UE_CLOGF(bDidTimeoutOnDecompressionFailiures, LogAudio, Error, "Timed out trying to decompress a looping sound after %d attempts; aborting.", VorbisReadFailiureTimeoutCVar);

			// If we're looping, then we need to make sure we wrap the stream chunks back to 0
			if (bLooping && !bDidTimeoutOnDecompressionFailiures)
			{
				NextStreamingChunkIndex = 0;
			}
			else
			{
				// Here we clear out the remainder of the buffer and bail.
				FMemory::Memzero(InDestination, BufferSize);
				break;
			}

			// Since we can't tell a streaming file to go back to the start of the stream (there is no seek) we have to close and reopen it.
			ov_callbacks Callbacks;
			Callbacks.read_func = OggReadStreaming;
			Callbacks.close_func = OggCloseStreaming;
			Callbacks.seek_func = NULL;	// Force streaming
			Callbacks.tell_func = NULL;	// Force streaming
			int Result = ov_open_callbacks(this, &VFWrapper->vf, NULL, 0, Callbacks);
			if (Result < 0)
			{
				LastErrorCode = Result;
				bHasError = true;
				UE_LOGF(LogAudio, Error, "FVorbisAudioInfo::StreamCompressedData, ov_open_callbacks error code: %d : %ls", Result, GetVorbisError(Result));
				break;
			}

			// else we start over to get the samples from the start of the compressed audio data
			continue;
		}
		else
		{
			TimesLoopedWithoutDecompressedAudio = 0;
		}

		InDestination += BytesActuallyRead;
		BufferSize -= BytesActuallyRead;
	}

	OutNumBytesStreamed = DestinationSize - BufferSize;
	return( bLooped );
}

bool FVorbisAudioInfo::HasError() const
{
	return Super::HasError() | bHasError;
}

void LoadVorbisLibraries()
{
	static const bool bDone = []() -> bool
	{
#if WITH_OGGVORBIS_DLL && WITH_OGGVORBIS
		//@todo if ogg is every ported to another platform, then use the platform abstraction to load these DLLs
		// Load the Ogg dlls
#  if _MSC_VER >= 1900
		FString VSVersion = "VS2015/";
#  else
#    error "Unsupported Visual Studio version."
#  endif
		FString PlatformString = "Win64";
		FString DLLNameStub = "_64.dll";

		// vorbis winarm64 libs are static for now, so no need to load anything
#if PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		FString RootOggPath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/Ogg/") / PlatformString / VSVersion / TEXT("arm64/");
		FString DLLToLoad = RootOggPath + TEXT("libogg") + DLLNameStub;
		void* LibOggHandle = FPlatformProcess::GetDllHandle(*DLLToLoad);
		verifyf(LibOggHandle, TEXT("Failed to load DLL %s"), *DLLToLoad);

		// Set that we successfully loaded everything so we can do nothing if it fails and avoid a crash
		bDllLoaded = !!LibOggHandle;

#else
		FString RootOggPath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/Ogg/") / PlatformString / VSVersion;
		FString RootVorbisPath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/Vorbis/") / PlatformString / VSVersion;

		FString DLLToLoad = RootOggPath + TEXT("libogg") + DLLNameStub;
		void* LibOggHandle = FPlatformProcess::GetDllHandle(*DLLToLoad);
		verifyf(LibOggHandle, TEXT("Failed to load DLL %s"), *DLLToLoad);
		// Load the Vorbis dlls
		DLLToLoad = RootVorbisPath + TEXT("libvorbis") + DLLNameStub;

		void* LibVorbisHandle = FPlatformProcess::GetDllHandle(*DLLToLoad);
		verifyf(LibVorbisHandle, TEXT("Failed to load DLL %s"), *DLLToLoad);
		DLLToLoad = RootVorbisPath + TEXT("libvorbisfile") + DLLNameStub;
		
		void* LibVorbisFileHandle = FPlatformProcess::GetDllHandle(*DLLToLoad);
		verifyf(LibVorbisFileHandle, TEXT("Failed to load DLL %s"), *DLLToLoad);

		// Set that we successfully loaded everything so we can do nothing if it fails and avoid a crash
		bDllLoaded = LibOggHandle && LibVorbisHandle && LibVorbisFileHandle;
#endif

		if (!bDllLoaded)
		{
			UE_LOGF(LogAudio, Error, "Failed to load lib vorbis libraries.");
		}
		else
		{
			UE_LOGF(LogAudioDebug, Display, "Lib vorbis DLL was dynamically loaded.");
		}
#elif WITH_OGGVORBIS
		bDllLoaded = true;
#endif	//WITH_OGGVORBIS_DLL
		return true;
	}();
	(void)bDone;
}

#endif		// WITH_OGGVORBIS

class VORBISAUDIODECODER_API FVorbisAudioDecoderModule : public IModuleInterface
{
public:	
	TUniquePtr<IAudioInfoFactory> Factory;

	virtual void StartupModule() override
	{
		LoadVorbisLibraries();

		constexpr FSimpleAudioInfoFactory::FCapabilities Caps = 
		{
			/*bSupportsSeeking*/ true,				
			/*bSupportsSeekableStreaming*/ false,	
		};
		Factory = MakeUnique<FSimpleAudioInfoFactory>([] { return new FVorbisAudioInfo(); }, Audio::NAME_OGG, Caps);
	}

	virtual void ShutdownModule() override {}
};

IMPLEMENT_MODULE(FVorbisAudioDecoderModule, VorbisAudioDecoder)
