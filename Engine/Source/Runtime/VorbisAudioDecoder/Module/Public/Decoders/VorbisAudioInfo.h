// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VorbisAudioInfo.h: Unreal audio vorbis decompression interface object.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeBool.h"
#include "AudioDecompress.h"

/**
 * Whether to use OggVorbis audio format.
 **/
#ifndef WITH_OGGVORBIS
	#define WITH_OGGVORBIS 0
#endif

/**
 * Whether to load OggVorbis DLLs
 **/
#ifndef WITH_OGGVORBIS_DLL
	#define WITH_OGGVORBIS_DLL (PLATFORM_WINDOWS)
#endif


namespace VorbisChannelInfo
{
	extern VORBISAUDIODECODER_API const int32 Order[8][8];
}

/**
 * Loads vorbis dlls
*/
VORBISAUDIODECODER_API void LoadVorbisLibraries();
#if WITH_OGGVORBIS

/** 
 * Helper class to parse ogg vorbis data
 */
class FVorbisAudioInfo : public ICompressedAudioInfo
{
public:
	VORBISAUDIODECODER_API FVorbisAudioInfo( void );
	VORBISAUDIODECODER_API virtual ~FVorbisAudioInfo( void );
	/** Emulate read from memory functionality */
	size_t			ReadMemory( void *ptr, uint32 size );
	int				SeekMemory( uint32 offset, int whence );
	int				CloseMemory( void );
	long			TellMemory( void );

	/** Emulate read from streaming functionality */
	size_t			ReadStreaming( void *ptr, uint32 size );
	int				CloseStreaming( void );

	/** Common info and data functions between ReadCompressedInfo/ReadCompressedData and StreamCompressedInfo/StreamCompressedData */
	/** Can't use ov_callbacks here so we have to pass in a void* for the callbacks instead */
	bool GetCompressedInfoCommon(void*	Callbacks, FSoundQualityInfo* QualityInfo);

	/** 
	 * Reads the header information of an ogg vorbis file
	 * 
	 * @param	Resource		Info about vorbis data
	 */
	VORBISAUDIODECODER_API virtual bool ReadCompressedInfo( const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, struct FSoundQualityInfo* QualityInfo ) override;
	/** 
	 * Decompresses ogg data to raw PCM data. 
	 * 
	 * @param	Destination	where to place the decompressed sound
	 * @param	bLooping	whether to loop the sound by seeking to the start, or pad the buffer with zeroes
	 * @param	BufferSize	number of bytes of PCM data to create
	 *
	 * @return	bool		true if the end of the data was reached (for both single shot and looping sounds)
	 */
	VORBISAUDIODECODER_API virtual bool ReadCompressedData( uint8* InDestination, bool bLooping, uint32 BufferSize ) override;
	VORBISAUDIODECODER_API virtual void SeekToTime( const float SeekTime ) override;
	VORBISAUDIODECODER_API virtual void SeekToFrame(const uint32 SeekFrame) override;
	/** 
	 * Decompress an entire ogg data file to a TArray
	 */
	VORBISAUDIODECODER_API virtual void ExpandFile( uint8* DstBuffer, struct FSoundQualityInfo* QualityInfo ) override;
	/** 
	 * Sets ogg to decode to half-rate
	 * 
	 * @param	Resource		Info about vorbis data
	 */
	VORBISAUDIODECODER_API virtual void EnableHalfRate( bool HalfRate ) override;
	virtual uint32 GetSourceBufferSize() const override { return SrcBufferDataSize; }

	virtual bool UsesVorbisChannelOrdering() const override { return true; }
	virtual int GetStreamBufferSize() const override { return MONO_PCM_BUFFER_SIZE; }

	// Additional overrides for streaming
	virtual bool SupportsStreaming() const override {return true;}
	virtual bool StreamCompressedData(uint8* InDestination, bool bLooping, uint32 BufferSize, int32& OutNumBytesStreamed) override;
	virtual int32 GetCurrentChunkIndex() const override {return CurrentStreamingChunkIndex;}
	virtual int32 GetCurrentChunkOffset() const override { return (CurrentStreamingChunksSize > 0)? BufferOffset % CurrentStreamingChunksSize : -1; }
	virtual bool HasError() const override;
	// End of ICompressedAudioInfo Interface

protected:
	VORBISAUDIODECODER_API virtual bool StreamCompressedInfoInternal(const TSharedRef<const FSoundWaveData>& InWaveData, struct FSoundQualityInfo* QualityInfo) override;

	friend class FAudioFormatOgg;
	VORBISAUDIODECODER_API int32 GetAudioDataStartOffset() const;
private:
	using Super = ICompressedAudioInfo; 
	const uint8* GetLoadedChunk(const TSharedRef<const FSoundWaveData>& InSoundWaveData, uint32 ChunkIndex, uint32& OutChunkSize);

	struct FVorbisFileWrapper* VFWrapper;
	const uint8* SrcBufferData;
	uint32 SrcBufferDataSize;
	uint32 BufferOffset;
	uint32 CurrentBufferChunkOffset;

	// In case Ogg Vorbis fails to return any compressed audio for an asset,
	// we use this counter to exit out of the decoder's while loop early.
	int32 TimesLoopedWithoutDecompressedAudio;

	/** Critical section used to prevent multiple threads accessing same ogg-vorbis file handles at the same time */
	mutable FCriticalSection VorbisCriticalSection;

	uint8 const* CurrentStreamingChunkData;
	int32 CurrentStreamingChunkIndex;
	int32 NextStreamingChunkIndex;
	uint32 CurrentStreamingChunksSize;
	bool bHeaderParsed;
	bool bHasError = false;
	int32 LastErrorCode = 0;

	// This handle is used to ensure that a chunk currently being decoded isn't evicted until we are done with it.
	FAudioChunkHandle CurCompressedChunkHandle;
};
#endif
