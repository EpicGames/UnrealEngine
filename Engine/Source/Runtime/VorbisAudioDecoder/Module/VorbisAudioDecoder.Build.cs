// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VorbisAudioDecoder : ModuleRules
{
    public VorbisAudioDecoder(ReadOnlyTargetRules Target) : base(Target)
    {
        ShortName = "VorbisAudioDecoder";

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "Engine"
            }
        );
        
        AddEngineThirdPartyPrivateStaticDependencies(
	        Target,
		        "UEOgg",
		        "Vorbis",
		        "VorbisFile"
        );        

	    PublicDefinitions.Add("WITH_OGGVORBIS=1");
    }
}
